// Copyright Epic Games, Inc. All Rights Reserved.

#include "MRMeshComponent.h"
#include "PrimitiveSceneProxy.h"
#include "DynamicMeshBuilder.h"
#include "LocalVertexFactory.h"
#include "Containers/ResourceArray.h"
#include "SceneManagement.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "RenderingThread.h"
#include "RHIResourceUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "BaseMeshReconstructorModule.h"
#include "MeshReconstructorBase.h"
#include "AI/NavigationSystemHelpers.h"
#include "PackedNormal.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "PhysicsPublic.h"

#include "Misc/RuntimeErrors.h"
#include "Engine/Engine.h"
#include "SceneView.h"
#include "UObject/UObjectThreadContext.h"
#include "Stats/Stats.h"
#include "UObject/ConstructorHelpers.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MRMeshComponent)

DECLARE_CYCLE_STAT(TEXT("MrMesh SetCollisionProfileName"), STAT_MrMesh_SetCollisionProfileName, STATGROUP_Physics);
DECLARE_CYCLE_STAT(TEXT("Update Collision"), STAT_UpdateCollision, STATGROUP_MRMESH);

#define DEBUG_BRICK_CULLING

#ifdef DEBUG_BRICK_CULLING
enum class ECullingDebugState
{
	Off,
	On,
	Paused,
};

static TAutoConsoleVariable<int32> CVarPauseMRMeshBrickCulling(
	TEXT("r.MrMesh.BrickCullingDebugState"),
	static_cast<int32>(ECullingDebugState::Off),
	TEXT("MR Mesh brick culling debug state: 0=off, 1=on, 2=paused"));
#endif //DEBUG_BRICK_CULLING

/** Support for non-interleaved data streams. */
template<typename DataType>
class FMRMeshVertexBuffer : public FVertexBuffer
{
public:
	int32 NumVerts = 0;
	void InitRHIWith(FRHICommandListBase& RHICmdList, const TArray<DataType>& PerVertexData )
	{
		NumVerts = PerVertexData.Num();
		VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("FMRMeshVertexBuffer"), EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource, MakeConstArrayView(PerVertexData));
	}
};

class FMRMeshIndexBuffer : public FIndexBuffer
{
public:
	int32 NumIndices = 0;
	void InitRHIWith(FRHICommandListBase& RHICmdList, const TArray<uint32>& Indices )
	{
		NumIndices = Indices.Num();
		IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FMRMeshIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Indices));
	}

	void InitRHIWith(FRHICommandListBase& RHICmdList, const TArray<uint16>& Indices)
	{
		NumIndices = Indices.Num();
		IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FMRMeshIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Indices));
	}
};


struct FMRMeshProxySection;

struct FMRMeshProxySection
{
	/** Which brick this section represents */
	IMRMesh::FBrickId BrickId;
	/** Position buffer */
	FMRMeshVertexBuffer<FVector3f> PositionBuffer;
	/** Texture coordinates buffer */
	FMRMeshVertexBuffer<FVector2D> UVBuffer;
	/** Tangent space buffer */
	FMRMeshVertexBuffer<FPackedNormal> TangentXZBuffer;
	/** We don't need color */
	FMRMeshVertexBuffer<FColor> ColorBuffer;
	/** Index buffer for this section */
	FMRMeshIndexBuffer IndexBuffer;
	/** Vertex factory for this section */
	FLocalVertexFactory VertexFactory;
	/** AABB for this section */
	FBox Bounds;

	FShaderResourceViewRHIRef PositionBufferSRV;
	FShaderResourceViewRHIRef UVBufferSRV;
	FShaderResourceViewRHIRef TangentXZBufferSRV;
	FShaderResourceViewRHIRef ColorBufferSRV;

	FMRMeshProxySection(IMRMesh::FBrickId InBrickId, ERHIFeatureLevel::Type InFeatureLevel)
		: BrickId(InBrickId)
		, VertexFactory(InFeatureLevel, "FMRMeshProxySection")
	{
	}

	void ReleaseResources()
	{
		PositionBuffer.ReleaseResource();
		UVBuffer.ReleaseResource();
		TangentXZBuffer.ReleaseResource();
		ColorBuffer.ReleaseResource();
		IndexBuffer.ReleaseResource();
		VertexFactory.ReleaseResource();
	}

	FMRMeshProxySection(const FLocalVertexFactory&) = delete;
	void operator==(const FLocalVertexFactory&) = delete;
};

static void InitVertexFactory(FLocalVertexFactory* VertexFactory, const FMRMeshProxySection& MRMeshSection)
{
	ENQUEUE_RENDER_COMMAND(InitMrMeshVertexFactory)(
		[VertexFactory, &MRMeshSection](FRHICommandListImmediate& RHICmdList)
	{
		check(IsInRenderingThread());

		// Initialize the vertex factory's stream components.
		FLocalVertexFactory::FDataType NewData;

		{
			NewData.PositionComponentSRV = MRMeshSection.PositionBufferSRV;
			NewData.PositionComponent = FVertexStreamComponent(&MRMeshSection.PositionBuffer, 0, sizeof(FVector3f), VET_Float3, EVertexStreamUsage::Default);
		}

		if (MRMeshSection.UVBuffer.NumVerts != 0)
		{
			NewData.TextureCoordinatesSRV = MRMeshSection.UVBufferSRV;
			NewData.TextureCoordinates.Add(FVertexStreamComponent(&MRMeshSection.UVBuffer, 0, sizeof(FVector2f), VET_Float2, EVertexStreamUsage::ManualFetch));
			NewData.NumTexCoords = 1;
		}

		if (MRMeshSection.TangentXZBuffer.NumVerts != 0)
		{
			NewData.TangentsSRV = MRMeshSection.TangentXZBufferSRV;
			NewData.TangentBasisComponents[0] = FVertexStreamComponent(&MRMeshSection.TangentXZBuffer, 0, 2 * sizeof(FPackedNormal), VET_PackedNormal, EVertexStreamUsage::ManualFetch);
			NewData.TangentBasisComponents[1] = FVertexStreamComponent(&MRMeshSection.TangentXZBuffer, sizeof(FPackedNormal), 2 * sizeof(FPackedNormal), VET_PackedNormal, EVertexStreamUsage::ManualFetch);
		}

		if (MRMeshSection.ColorBuffer.NumVerts != 0)
		{
			NewData.ColorComponentsSRV = MRMeshSection.ColorBufferSRV;
			NewData.ColorComponent = FVertexStreamComponent(&MRMeshSection.ColorBuffer, 0, sizeof(FColor), VET_Color, EVertexStreamUsage::ManualFetch);
		}

		VertexFactory->SetData(RHICmdList, NewData);
		VertexFactory->InitResource(RHICmdList);
	});
}


class FMRMeshProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FMRMeshProxy(const UMRMeshComponent* InComponent)
		: FPrimitiveSceneProxy(InComponent, InComponent->GetFName())
		, MaterialToUse(InComponent->GetMaterialToUse())
		, FeatureLevel(GetScene().GetFeatureLevel())
		, bEnableOcclusion(InComponent->GetEnableMeshOcclusion())
		, bUseWireframe(InComponent->GetUseWireframe())
	{
	}
	
	virtual ~FMRMeshProxy()
	{
		for (FMRMeshProxySection* Section : ProxySections)
		{
			if (Section != nullptr)
			{
				Section->ReleaseResources();
				delete Section;
			}
		}
	}

	void RenderThread_UploadNewSection(IMRMesh::FSendBrickDataArgs Args)
	{
		check(IsInRenderingThread() || IsInRHIThread());
		FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

		FMRMeshProxySection* NewSection = new FMRMeshProxySection(Args.BrickId, FeatureLevel);
		ProxySections.Add(NewSection);

		// Vulkan requires that all the buffers be full.
		const int32 NumVerts = Args.PositionData.Num();
		check((NumVerts == Args.ColorData.Num()));
		check((NumVerts == Args.UVData.Num()));
		check((NumVerts * 2) == Args.TangentXZData.Num());

		// POSITION BUFFER
		{
			NewSection->PositionBuffer.InitResource(RHICmdList);
			NewSection->PositionBuffer.InitRHIWith(RHICmdList, Args.PositionData);
			NewSection->PositionBufferSRV = RHICmdList.CreateShaderResourceView(
				NewSection->PositionBuffer.VertexBufferRHI, 
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_FLOAT));
		}

		// TEXTURE COORDS BUFFER
		{
			NewSection->UVBuffer.InitResource(RHICmdList);
			if (Args.UVData.Num())
			{
				NewSection->UVBuffer.InitRHIWith(RHICmdList, Args.UVData);
				NewSection->UVBufferSRV = RHICmdList.CreateShaderResourceView(
					NewSection->UVBuffer.VertexBufferRHI,
					FRHIViewDesc::CreateBufferSRV()
						.SetType(FRHIViewDesc::EBufferType::Typed)
						.SetFormat(PF_G32R32F));
			}
		}

		// TANGENTS BUFFER
		{
			NewSection->TangentXZBuffer.InitResource(RHICmdList);
			if (Args.TangentXZData.Num())
			{
				NewSection->TangentXZBuffer.InitRHIWith(RHICmdList, Args.TangentXZData);
			}

			if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
			{
				NewSection->TangentXZBufferSRV = RHICmdList.CreateShaderResourceView(
					NewSection->TangentXZBuffer.VertexBufferRHI,
					FRHIViewDesc::CreateBufferSRV()
						.SetType(FRHIViewDesc::EBufferType::Typed)
						.SetFormat(PF_R8G8B8A8_SNORM));
			}
		}

		// COLOR
		{
			NewSection->ColorBuffer.InitResource(RHICmdList);
			if (Args.ColorData.Num())
			{
				NewSection->ColorBuffer.InitRHIWith(RHICmdList, Args.ColorData);
				NewSection->ColorBufferSRV = RHICmdList.CreateShaderResourceView(
					NewSection->ColorBuffer.VertexBufferRHI,
					FRHIViewDesc::CreateBufferSRV()
						.SetType(FRHIViewDesc::EBufferType::Typed)
						.SetFormat(PF_R8G8B8A8));
			}
		}

		// INDEX BUFFER
		{
			NewSection->IndexBuffer.InitResource(RHICmdList);
			NewSection->IndexBuffer.InitRHIWith(RHICmdList, Args.Indices);
		}

		// VERTEX FACTORY
		{
			InitVertexFactory(&NewSection->VertexFactory, *NewSection);
		}

		// BOUNDS
		NewSection->Bounds = Args.Bounds;
	}

	bool RenderThread_RemoveSection(IMRMesh::FBrickId BrickId)
	{
		check(IsInRenderingThread() || IsInRHIThread());
		for (int32 i = 0; i < ProxySections.Num(); ++i)
		{
			if (ProxySections[i]->BrickId == BrickId)
			{
				ProxySections[i]->ReleaseResources();
				delete ProxySections[i];
				ProxySections.RemoveAtSwap(i);
				return true;
			}
		}
		return false;
	}

	void RenderThread_RemoveAllSections()
	{
		check(IsInRenderingThread() || IsInRHIThread());
		for (int32 i = ProxySections.Num()-1; i >=0; i--)
		{
			ProxySections[i]->ReleaseResources();
			delete ProxySections[i];
			ProxySections.RemoveAtSwap(i);
		}
	}

	void RenderThread_SetMaterial(bool bInUseWireframe, UMaterialInterface* Material)
	{
		if (ensure(Material))
		{
			bUseWireframe = bInUseWireframe;
			MaterialToUse = Material;
#if WITH_EDITOR
			// When changing materials in the editor we need to keep the verification
			// materials set in sync to satisfy the internal class invariants. This also
			// avoids validation errors when generating the mesh batches.
			SetUsedMaterialForVerification(TArray<UMaterialInterface*>(&MaterialToUse, 1));
#endif
		}
	}

	void SetEnableMeshOcclusion(bool bEnable)
	{
		bEnableOcclusion = bEnable;
	}

private:
	//~ FPrimitiveSceneProxy

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, class FMeshElementCollector& Collector) const override
	{
		static const FBoxSphereBounds InfiniteBounds(FSphere(FVector::ZeroVector, HALF_WORLD_MAX));

#ifdef DEBUG_BRICK_CULLING
		TMap<IMRMesh::FBrickId, TTuple<FBox, bool>> NewVisDataByBrickId;
		ECullingDebugState CullingDebugState =
			static_cast<ECullingDebugState>(CVarPauseMRMeshBrickCulling.GetValueOnRenderThread());
#endif //DEBUG_BRICK_CULLING

		// Iterate over sections
		for (const FMRMeshProxySection* Section : ProxySections)
		{
			if (Section != nullptr)
			{
				FMaterialRenderProxy* MaterialProxy = MaterialToUse->GetRenderProxy();

				// For each view..
				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					if (VisibilityMap & (1 << ViewIndex))
					{
						const FSceneView* View = Views[ViewIndex];
						bool IsVisible = Section->Bounds.GetExtent().IsNearlyZero() || View->ViewFrustum.
							IntersectBox(Section->Bounds.GetCenter(), Section->Bounds.GetExtent());

#ifdef DEBUG_BRICK_CULLING
						switch (CullingDebugState)
						{
							case ECullingDebugState::Off:
								break;
							case ECullingDebugState::On:
								NewVisDataByBrickId.Add(Section->BrickId, MakeTuple(Section->Bounds, IsVisible));
								break;
							case ECullingDebugState::Paused:
								auto OldVisData = OldVisDataByBrickId.Find(Section->BrickId);
								if (OldVisData)
								{
									NewVisDataByBrickId.Add(Section->BrickId, *OldVisData);

									// Easier to see what's culled if mesh mimics culling pause state
									IsVisible = OldVisData->Value;
								}
								else
								{
									IsVisible = false;
								}
								break;
						}
#endif //DEBUG_BRICK_CULLING

						if (IsVisible)
						{
							// Draw the mesh.
							FMeshBatch& Mesh = Collector.AllocateMesh();
							FMeshBatchElement& BatchElement = Mesh.Elements[0];
							BatchElement.IndexBuffer = &Section->IndexBuffer;
							Mesh.bWireframe = bUseWireframe;
							Mesh.bUseAsOccluder = bEnableOcclusion;
							Mesh.bUseForDepthPass = bEnableOcclusion;

							Mesh.VertexFactory = &Section->VertexFactory;
							Mesh.MaterialRenderProxy = MaterialProxy;

							FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
							DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), GetLocalToWorld(), GetLocalToWorld(), InfiniteBounds, InfiniteBounds, true, false, AlwaysHasVelocity());
							BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

							BatchElement.FirstIndex = 0;
							BatchElement.NumPrimitives = Section->IndexBuffer.NumIndices / 3;
							BatchElement.MinVertexIndex = 0;
							BatchElement.MaxVertexIndex = Section->PositionBuffer.NumVerts - 1;
							Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
							Mesh.Type = PT_TriangleList;
							Mesh.DepthPriorityGroup = SDPG_World;
							Mesh.bCanApplyViewModeOverrides = false;
							Collector.AddMesh(ViewIndex, Mesh);
						}
					}
				}
			}
		}

#ifdef DEBUG_BRICK_CULLING
		OldVisDataByBrickId = NewVisDataByBrickId;

		if (CullingDebugState != ECullingDebugState::Off)
		{
			const FColor ColorGray(0x7f, 0x7f, 0x7f, 0xff);

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					for (const auto& BrickVisibility : OldVisDataByBrickId)
					{
						const auto& BrickBounds = BrickVisibility.Value.Key;
						FColor BoundsColor(BrickVisibility.Value.Value ? FColor::Green : ColorGray);
						FPrimitiveDrawInterface *PDI = Collector.GetPDI(ViewIndex);
						DrawWireBox(PDI, BrickBounds, BoundsColor, (uint8)GetDepthPriorityGroup(Views[ViewIndex]));
					}
				}
			}
		}
#endif //DEBUG_BRICK_CULLING
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bDynamicRelevance = true;
		// If there is a material set that is not the default material, then this wants to be rendered in the main pass
		Result.bRenderInMainPass = (bUseWireframe || MaterialToUse != UMaterial::GetDefaultMaterial(MD_Surface)) && ShouldRenderInMainPass();
		Result.bRenderInDepthPass = bEnableOcclusion;
		Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		TMicRecursionGuard RecursionGuard;
		Result.bSeparateTranslucency = MaterialToUse->GetMaterial_Concurrent(RecursionGuard)->TranslucencyPass == MTP_AfterDOF;
		//MaterialRelevance.SetPrimitiveViewRelevance(Result);
		return Result;
	}
	virtual uint32 GetMemoryFootprint(void) const
	{
		return(sizeof(*this) + GetAllocatedSize());
	}

	uint32 GetAllocatedSize(void) const
	{
		return((uint32)FPrimitiveSceneProxy::GetAllocatedSize());
	}

private:
	TArray<FMRMeshProxySection*> ProxySections;
	UMaterialInterface* MaterialToUse;
	ERHIFeatureLevel::Type FeatureLevel;
	bool bEnableOcclusion;
	bool bUseWireframe;

#ifdef DEBUG_BRICK_CULLING
	mutable TMap<IMRMesh::FBrickId, TTuple<FBox, bool>> OldVisDataByBrickId;
#endif //DEBUG_BRICK_CULLING
};


UMRMeshComponent::UMRMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bEnableOcclusion(false)
	, bUseWireframe(false)
{
}

void UMRMeshComponent::BeginPlay()
{
	Super::BeginPlay();

	SetCustomNavigableGeometry(bCanEverAffectNavigation ? EHasCustomNavigableGeometry::Yes : EHasCustomNavigableGeometry::No);
}

void UMRMeshComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	ClearAllBrickData();

	Super::EndPlay(EndPlayReason);
}

void UMRMeshComponent::OnActorEnableCollisionChanged()
{
	for (auto& BodyHolderElement : BodyHolders)
	{
		BodyHolderElement->BodyInstance.UpdatePhysicsFilterData();
	}

	Super::OnActorEnableCollisionChanged();
}

bool UMRMeshComponent::ShouldCreatePhysicsState() const
{
	// This component does not use the default physics state creation.  It creates in response to meshing data delivered via SendBrickData.
	return false;
}

void UMRMeshComponent::SetCollisionEnabled(ECollisionEnabled::Type NewType)
{
	if (BodyInstance.GetCollisionEnabled() != NewType)
	{
		for (auto& BodyHolderElement : BodyHolders)
		{
			BodyHolderElement->BodyInstance.SetCollisionEnabled(NewType);
		}

		if (IsRegistered() && BodyInstance.bSimulatePhysics && !IsWelded())
		{
			for (auto& BodyHolderElement : BodyHolders)
			{
				BodyHolderElement->BodyInstance.ApplyWeldOnChildren();
			}
		}
	}

	Super::SetCollisionEnabled(NewType);
}

void UMRMeshComponent::SetCollisionProfileName(FName InCollisionProfileName, bool bUpdateOverlaps)
{
	SCOPE_CYCLE_COUNTER(STAT_MrMesh_SetCollisionProfileName);

	FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
	if (ThreadContext.ConstructedObject == this)
	{
		// If we are in our constructor, defer setup until PostInitProperties as derived classes
		for (auto& BodyHolderElement : BodyHolders)
		{
			BodyHolderElement->BodyInstance.SetCollisionProfileNameDeferred(InCollisionProfileName);
		}
	}
	else
	{
		for (auto& BodyHolderElement : BodyHolders)
		{
			BodyHolderElement->BodyInstance.SetCollisionProfileName(InCollisionProfileName);
		}
	}

	Super::SetCollisionProfileName(InCollisionProfileName, bUpdateOverlaps);
}

void UMRMeshComponent::SetCollisionObjectType(ECollisionChannel Channel)
{
	for (auto& BodyHolderElement : BodyHolders)
	{
		BodyHolderElement->BodyInstance.SetObjectType(Channel);
	}

	Super::SetCollisionObjectType(Channel);
}

void UMRMeshComponent::SetCollisionResponseToChannel(ECollisionChannel Channel, ECollisionResponse NewResponse)
{
	for (auto& BodyHolderElement : BodyHolders)
	{
		BodyHolderElement->BodyInstance.SetResponseToChannel(Channel, NewResponse);
	}

	Super::SetCollisionResponseToChannel(Channel, NewResponse);
}

void UMRMeshComponent::SetCollisionResponseToAllChannels(enum ECollisionResponse NewResponse)
{
	for (auto& BodyHolderElement : BodyHolders)
	{
		BodyHolderElement->BodyInstance.SetResponseToAllChannels(NewResponse);
	}

	Super::SetCollisionResponseToAllChannels(NewResponse);
}

void UMRMeshComponent::SetCollisionResponseToChannels(const FCollisionResponseContainer& NewResponses)
{
	for (auto& BodyHolderElement : BodyHolders)
	{
		BodyHolderElement->BodyInstance.SetResponseToChannels(NewResponses);
	}

	Super::SetCollisionResponseToChannels(NewResponses);
}

void UMRMeshComponent::UpdatePhysicsToRBChannels()
{
	for (auto& BodyHolderElement : BodyHolders)
	{
		if (BodyHolderElement->BodyInstance.IsValidBodyInstance())
		{
			BodyHolderElement->BodyInstance.UpdatePhysicsFilterData();
		}
	}

	Super::UpdatePhysicsToRBChannels();
}

void UMRMeshComponent::SetWalkableSlopeOverride(const FWalkableSlopeOverride& NewOverride)
{
	for (auto& BodyHolderElement : BodyHolders)
	{
		if (BodyHolderElement->BodyInstance.IsValidBodyInstance())
		{
			BodyHolderElement->BodyInstance.SetWalkableSlopeOverride(NewOverride);
		}
	}

	Super::SetWalkableSlopeOverride(NewOverride);
}

FPrimitiveSceneProxy* UMRMeshComponent::CreateSceneProxy()
{
	// The render thread owns the memory, so if this function is
	// being called, it's safe to just re-allocate.
	return new FMRMeshProxy(this);
}

void UMRMeshComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials /*= false*/) const
{
	if (Material != nullptr)
	{
		OutMaterials.Add(Material);
	}
	if (WireframeMaterial != nullptr)
	{
		OutMaterials.Add(WireframeMaterial);
	}
}

FBoxSphereBounds UMRMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds(FSphere(FVector::ZeroVector, HALF_WORLD_MAX));
}

void UMRMeshComponent::SendBrickData(IMRMesh::FSendBrickDataArgs Args)
{
	auto BrickDataTask = FSimpleDelegateGraphTask::FDelegate::CreateUObject(this, &UMRMeshComponent::SendBrickData_Internal, Args);

	DECLARE_CYCLE_STAT(TEXT("UMRMeshComponent.SendBrickData"),
		STAT_UMRMeshComponent_SendBrickData,
		STATGROUP_MRMESH);

	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(BrickDataTask, GET_STATID(STAT_UMRMeshComponent_SendBrickData), nullptr, ENamedThreads::GameThread);
}

void UMRMeshComponent::ClearAllBrickData()
{
	auto ClearBrickDataTask = FSimpleDelegateGraphTask::FDelegate::CreateUObject(this, &UMRMeshComponent::ClearAllBrickData_Internal);

	DECLARE_CYCLE_STAT(TEXT("UMRMeshComponent.ClearAllBrickData"),
	STAT_UMRMeshComponent_ClearAllBrickData,
		STATGROUP_MRMESH);

	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(ClearBrickDataTask, GET_STATID(STAT_UMRMeshComponent_ClearAllBrickData), nullptr, ENamedThreads::GameThread);
}

void UMRMeshComponent::SendBrickData_Internal(IMRMesh::FSendBrickDataArgs Args)
{
	check(IsInGameThread());
	const bool bHasBrickData = Args.Indices.Num() > 0 && Args.PositionData.Num() > 0;
	
	OnBrickDataUpdatedDelegate.Broadcast(this, Args);

	UE_LOGF(LogMrMesh, Log, "SendBrickData_Internal() processing brick %llu with %i triangles", Args.BrickId, Args.Indices.Num() / 3);
	
	// Collision generation.
	if (IsValidChecked(this) && !bNeverCreateCollisionMesh)
	{
			SCOPE_CYCLE_COUNTER(STAT_UpdateCollision);

			UWorld* World = GetWorld();
			if (World && World->GetPhysicsScene())
			{
				int32 BodyIndex = BodyIds.Find(Args.BrickId);

				if (bHasBrickData)
				{
					bPhysicsStateCreated = true;

					if (BodyIndex == INDEX_NONE)
					{
						TObjectPtr<UMRMeshBodyHolder> NewBodyHolder = NewObject<UMRMeshBodyHolder>(this, NAME_None, (IsTemplate() ? RF_Public | RF_ArchetypeObject : RF_NoFlags));
						NewBodyHolder->Initialize(Args.BrickId);
						BodyHolders.Add(NewBodyHolder);
						BodyIndex = BodyIds.Add(Args.BrickId);
					}

					UMRMeshBodyHolder& BodyHolder = *BodyHolders[BodyIndex].Get();
					check(BodyHolder.BrickId == Args.BrickId);

					// Interrupt any in progress cook for this brick.
					BodyHolder.AbortCook();

					BodyHolder.Update(Args);

					const bool bUseAsyncCook = bUseAsyncCooking && World->IsGameWorld();
					if (bUseAsyncCook)
					{
						BodyHolder.bCookInProgress = true;
						BodyHolder.BodySetup->CreatePhysicsMeshesAsync(FOnAsyncPhysicsCookFinished::CreateUObject(&BodyHolder, &UMRMeshBodyHolder::FinishPhysicsAsyncCook, BodyHolder.BodySetup.Get()));
					}
					else
					{
						// Also we want cooked data for this
						BodyHolder.BodySetup->bHasCookedCollisionData = true;
						BodyHolder.BodySetup->InvalidatePhysicsData();
						BodyHolder.BodySetup->CreatePhysicsMeshes();
						RecreatePhysicsState();

						SuggestNavMeshUpdate();
					}
				}
				else // !bHasBrickData
				{
					if (BodyIndex != INDEX_NONE)
					{
						RemoveBodyInstance(BodyIndex);

						SuggestNavMeshUpdate();
					}
					else
					{
						// This brick already doesn't exist, so no work to be done.
					}
				}

			}
			else
			{
				// In A WORLD without physics you cannot COOK collision!
				UE_LOGF(LogMrMesh, Warning, "UMRMeshComponent::SendBrickData_Internal tried to cook collision without a world or in a world without physics!  No collision cooked.");
			}
	}

	// Rendering generation
	if (bCreateMeshProxySections)
	{
		if (SceneProxy != nullptr && GIsThreadedRendering)
		{
			// Graphics update
			UMRMeshComponent* This = this;
			ENQUEUE_RENDER_COMMAND(FSendBrickDataLambda)(
				[This, Args, bHasBrickData](FRHICommandListImmediate& RHICmdList)
				{
					FMRMeshProxy* MRMeshProxy = static_cast<FMRMeshProxy*>(This->SceneProxy);
					if (MRMeshProxy)
					{
						MRMeshProxy->RenderThread_RemoveSection(Args.BrickId);

						if (bHasBrickData)
						{
							MRMeshProxy->RenderThread_UploadNewSection(Args);
						}
					}
				}
			);
		}
	}
}

void UMRMeshComponent::RemoveBodyInstance(int32 BodyIndex)
{
	auto& BH = BodyHolders[BodyIndex];
	BH->Cleanup();
	BodyHolders.RemoveAtSwap(BodyIndex);
	BodyIds.RemoveAtSwap(BodyIndex);
}

void UMRMeshComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	Super::OnUpdateTransform(UpdateTransformFlags, Teleport);

	for (auto& BH : BodyHolders)
	{
		BH->BodyInstance.SetBodyTransform(GetComponentTransform(), Teleport);
		BH->BodyInstance.UpdateBodyScale(GetComponentTransform().GetScale3D());
	}
}

void UMRMeshComponent::ClearAllBrickData_Internal()
{
	check(IsInGameThread());

	for (int32 i = BodyIds.Num()-1; i >= 0; i--)
	{
		RemoveBodyInstance(i);
	}

	// Graphics update
	UMRMeshComponent* This = this;
	ENQUEUE_RENDER_COMMAND(FClearAllBricksLambda)(
		[This](FRHICommandListImmediate& RHICmdList)
		{
			FMRMeshProxy* MRMeshProxy = static_cast<FMRMeshProxy*>(This->SceneProxy);
			if (MRMeshProxy)
			{
				MRMeshProxy->RenderThread_RemoveAllSections();
			}
		}
	);

	if (OnClear().IsBound())
	{
		OnClear().Broadcast();
	}
}

void UMRMeshComponent::SetMaterial(int32 ElementIndex, class UMaterialInterface* InMaterial)
{
	if (Material != InMaterial)
	{
		Material = InMaterial;
		MarkRenderDynamicDataDirty();
	}
}

UMaterialInterface* UMRMeshComponent::GetMaterial(int32 ElementIndex) const
{
	return Material;
}

void UMRMeshComponent::SetWireframeMaterial(class UMaterialInterface* InMaterial)
{
	if (WireframeMaterial != InMaterial)
	{
		WireframeMaterial = InMaterial;
		MarkRenderDynamicDataDirty();
	}
}

void UMRMeshComponent::SendRenderDynamicData_Concurrent()
{
	Super::SendRenderDynamicData_Concurrent();

	if (SceneProxy)
	{
		// Enqueue command to send to render thread
		UMRMeshComponent* This = this;
		auto InMaterial = GetMaterialToUse();
		bool bUseWireframeLocal = bUseWireframe;
		ENQUEUE_RENDER_COMMAND(FSetMaterialLambda)(
		[This, bUseWireframeLocal, InMaterial](FRHICommandListImmediate& RHICmdList)
		{
			FMRMeshProxy* MRMeshProxy = static_cast<FMRMeshProxy*>(This->SceneProxy);
			if (MRMeshProxy)
			{
				MRMeshProxy->RenderThread_SetMaterial(bUseWireframeLocal, InMaterial);
			}
		});
	}
}

bool UMRMeshComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	check(bHasCustomNavigableGeometry);
	
	for (auto& BH : BodyHolders)
	{
		check(BH->BodySetup);
		GeomExport.ExportRigidBodySetup(*BH->BodySetup, GetComponentTransform());
	}

	return false;
}

void UMRMeshComponent::SuggestNavMeshUpdate()
{
	bNavMeshUpdateSuggested = true;

	if (bUpdateNavMeshOnMeshUpdate && bHasCustomNavigableGeometry)
	{
		bNavMeshUpdateSuggested = false;
		UpdateNavigationData();
	}
}

void UMRMeshComponent::RequestNavMeshUpdate()
{
	if (bNavMeshUpdateSuggested == true)
	{
		ForceNavMeshUpdate();
	}
}

void UMRMeshComponent::ForceNavMeshUpdate()
{
	if (bHasCustomNavigableGeometry)
	{
		bNavMeshUpdateSuggested = false;
		UpdateNavigationData();
	}
	else
	{
		UE_LOGF(LogMrMesh, Log, "ForceNavMeshUpdate() called, but this MRMesh component has bCanEverAffectNavigation==false.  Ignoring forced update.");
	}
}

void UMRMeshComponent::Clear()
{
	ClearAllBrickData();
	UE_LOGF(LogMrMesh, Log, "Clearing all brick data");
}

struct FMeshArrayHolder :
	public IMRMesh::FBrickDataReceipt
{
	TArray<FVector3f> Vertices;
	TArray<MRMESH_INDEX_TYPE> Indices;
	// Super wasteful of memory and perf, but the vertex factory requires these to be filled
	// @todo Write a vertex factory that doesn't need all this overhead
	TArray<FVector2D> BogusUVs;
	TArray<FPackedNormal> BogusTangents;
	TArray<FColor> BogusColors;

	FMeshArrayHolder(TArray<FVector3f>& InVertices, TArray<MRMESH_INDEX_TYPE>& InIndices, TArray<FVector2D>& UVData, TArray<FPackedNormal>& TangentXZData, TArray<FColor>& ColorData)
		: Indices(MoveTemp(InIndices))
	{
		const int32 CurrentNumVertices = InVertices.Num();

		// This constructor is a bit slower because it has to copy the vertices one by one to convert them from float to double.
		Vertices.AddUninitialized(CurrentNumVertices);
		for (int i = 0; i < CurrentNumVertices; i++)
		{
			Vertices[i] = InVertices[i];
		}

		if (UVData.Num() == CurrentNumVertices)
		{
			BogusUVs = MoveTemp(UVData);
		}
		else
		{
			BogusUVs.AddZeroed(CurrentNumVertices);
		}

		if (ColorData.Num() == CurrentNumVertices)
		{
			BogusColors = MoveTemp(ColorData);
		}
		else
		{
			BogusColors.AddZeroed(CurrentNumVertices);
		}

		if (TangentXZData.Num() == CurrentNumVertices * 2)
		{
			BogusTangents = MoveTemp(TangentXZData);
		}
		else
		{
			BogusTangents.AddZeroed(CurrentNumVertices * 2);
		}
	}

	FMeshArrayHolder(TArray<FVector>& InVertices, TArray<MRMESH_INDEX_TYPE>& InIndices, TArray<FVector2D>& UVData, TArray<FPackedNormal>& TangentXZData, TArray<FColor>& ColorData)
		: Vertices(MoveTemp(InVertices))
		, Indices(MoveTemp(InIndices))
	{
		const int32 CurrentNumVertices = Vertices.Num();
		
		if (UVData.Num() == CurrentNumVertices)
		{
			BogusUVs = MoveTemp(UVData);
		}
		else
		{
			BogusUVs.AddZeroed(CurrentNumVertices);
		}
		
		if (ColorData.Num() == CurrentNumVertices)
		{
			BogusColors = MoveTemp(ColorData);
		}
		else
		{
			BogusColors.AddZeroed(CurrentNumVertices);
		}
		
		if (TangentXZData.Num() == CurrentNumVertices * 2)
		{
			BogusTangents = MoveTemp(TangentXZData);
		}
		else
		{
			BogusTangents.AddZeroed(CurrentNumVertices * 2);
		}
	}
};

void UMRMeshComponent::UpdateMesh(const FVector& InLocation, const FQuat& InRotation, const FVector& Scale, TArray<FVector3f>& Vertices, TArray<MRMESH_INDEX_TYPE>& Indices, TArray<FVector2D> UVData, TArray<FPackedNormal> TangentXZData, TArray<FColor> ColorData)
{
	SetRelativeLocationAndRotation(InLocation, InRotation);
	SetRelativeScale3D(Scale);

	// Create our struct that will hold the data until the render thread is done with it
	TSharedPtr<FMeshArrayHolder, ESPMode::ThreadSafe> MeshHolder = MakeShared<FMeshArrayHolder>(Vertices, Indices, UVData, TangentXZData, ColorData);
	// NOTE: Indices are empty due to MoveTemp()!!!
	 
	// Set a valid bounding box so meshes get correctly culled.
	FBox bounds = FBox(ForceInit);
	const int Num = Vertices.Num();
	for (int i = 0; i < Num; i++)
	{
		bounds += (FVector)Vertices[i];
	}
	bounds = bounds.TransformBy(FTransform(InRotation.Rotator(), InLocation, Scale));

	SendBrickData_Internal(IMRMesh::FSendBrickDataArgs
		{
			MeshHolder,
			0,
			MeshHolder->Vertices,
			MeshHolder->BogusUVs,
			MeshHolder->BogusTangents,
			MeshHolder->BogusColors,
			MeshHolder->Indices,
			bounds
		}
	);
}

void UMRMeshComponent::UpdateMesh(const FVector& InLocation, const FQuat& InRotation, const FVector& Scale, TArray<FVector>& Vertices, TArray<MRMESH_INDEX_TYPE>& Indices, TArray<FVector2D> UVData, TArray<FPackedNormal> TangentXZData, TArray<FColor> ColorData)
{
	SetRelativeLocationAndRotation(InLocation, InRotation);
	SetRelativeScale3D(Scale);

	// Set a valid bounding box so meshes get correctly culled.
	FBox bounds = FBox(Vertices).TransformBy(FTransform(InRotation.Rotator(), InLocation, Scale));

	// Create our struct that will hold the data until the render thread is done with it
	TSharedPtr<FMeshArrayHolder, ESPMode::ThreadSafe> MeshHolder(new FMeshArrayHolder(Vertices, Indices, UVData, TangentXZData, ColorData));
	// NOTE: Vertices and Indices are empty due to MoveTemp()!!!

	SendBrickData_Internal(IMRMesh::FSendBrickDataArgs
		{
			MeshHolder,
			0,
			MeshHolder->Vertices,
			MeshHolder->BogusUVs,
			MeshHolder->BogusTangents,
			MeshHolder->BogusColors,
			MeshHolder->Indices,
			bounds
		}
	);
}

void UMRMeshComponent::SetEnableMeshOcclusion(bool bEnable)
{
	bEnableOcclusion = bEnable;

	// Update bEnableOcclusion on the SceneProxy, as well.
	if (SceneProxy)
	{
		UMRMeshComponent* This = this;
		ENQUEUE_RENDER_COMMAND(FSetEnableMeshOcclusionLambda)(
			[This,bEnable](FRHICommandListImmediate& RHICmdList)
			{
				FMRMeshProxy* MRMeshProxy = static_cast<FMRMeshProxy*>(This->SceneProxy);
				if (MRMeshProxy)
				{
					MRMeshProxy->SetEnableMeshOcclusion(bEnable);
				}
			}
		);
	}
}

void UMRMeshComponent::SetWireframeColor(const FLinearColor& InColor)
{
	WireframeColor = InColor;
	if (auto MaterialInstance = Cast<UMaterialInstanceDynamic>(WireframeMaterial))
	{
		static const FName ParamName("Color");
		MaterialInstance->SetVectorParameterValue(ParamName, InColor);
		MarkRenderDynamicDataDirty();
	}
	else
	{
		WireframeMaterial = UMaterialInstanceDynamic::Create(WireframeMaterial, this);
		SetWireframeColor(InColor);
	}
}

void UMRMeshComponent::SetUseWireframe(bool InbUseWireframe)
{
	bUseWireframe = InbUseWireframe;
	MarkRenderDynamicDataDirty();
}

UMaterialInterface* UMRMeshComponent::GetMaterialToUse() const
{
	if (bUseWireframe && WireframeMaterial)
	{
		return WireframeMaterial;
	}
	else if (Material)
	{
		return Material;
	}
	else
	{
		return UMaterial::GetDefaultMaterial(MD_Surface);
	}
}

void UMRMeshBodyHolder::Initialize(IMRMesh::FBrickId InBrickId)
{
	check(BodySetup == nullptr);

	BrickId = InBrickId;

	BodySetup = NewObject<UBodySetup>(this, NAME_None, (IsTemplate() ? RF_Public | RF_ArchetypeObject : RF_NoFlags));
	BodySetup->BodySetupGuid = FGuid::NewGuid();
	BodySetup->bGenerateMirroredCollision = false;
	BodySetup->bHasCookedCollisionData = true;
	//BodySetup->bDoubleSidedGeometry = true;
	BodySetup->CollisionTraceFlag = CTF_UseComplexAsSimple;

	bCookInProgress = false;
}

void UMRMeshBodyHolder::Update(const IMRMesh::FSendBrickDataArgs& Args)
{
	check(BodySetup != nullptr);
	check(BrickId == Args.BrickId);
	BrickDataReceipt = Args.BrickDataReceipt; // This would release hold on any previous receipt.
	PositionData = &Args.PositionData;
	Indices = &Args.Indices;
	Bounds = Args.Bounds;
}

void UMRMeshBodyHolder::AbortCook()
{
	if (bCookInProgress)
	{
		BodySetup->AbortPhysicsMeshAsyncCreation();
		bCookInProgress = false;

		ReleaseArgData();
	}
}

void UMRMeshBodyHolder::ReleaseArgData()
{
	// Null all the raw pointers and release our hold on the receipt.
	PositionData = nullptr;
	Indices = nullptr;
	BrickDataReceipt.Reset();

	// Leave the POD members alone.
}

void UMRMeshBodyHolder::Cleanup()
{
	AbortCook();
	ReleaseArgData();

	BodyInstance.TermBody();
}

bool UMRMeshBodyHolder::GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const
{
	if (BrickDataReceipt.IsValid() && Indices->Num() > 0)
	{
		OutTriMeshEstimates.VerticeCount = PositionData->Num();
	}

	return true;
}

bool UMRMeshBodyHolder::GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData)
{
	if (BrickDataReceipt.IsValid() && Indices->Num() > 0)
	{
		// Copy the vertices
		CollisionData->Vertices.AddUninitialized(PositionData->Num());
		for (auto Position = 0; Position < PositionData->Num(); ++Position)
		{
			CollisionData->Vertices[Position] = (FVector3f)(*PositionData)[Position];
		}

		// Copy the indices
		const auto& IndicesR = *Indices;
		CollisionData->Indices.Reset(IndicesR.Num() / 3);
		for (auto Index = 0; Index < IndicesR.Num(); Index += 3)
		{
			FTriIndices Face;
			Face.v0 = IndicesR[Index];
			Face.v1 = IndicesR[Index + 1];
			Face.v2 = IndicesR[Index + 2];
			CollisionData->Indices.Add(Face);
		}

		CollisionData->bFlipNormals = true;
		CollisionData->bDeformableMesh = true;
		CollisionData->bFastCook = true;

		return true;
	}

	return false;
}
bool UMRMeshBodyHolder::ContainsPhysicsTriMeshData(bool InUseAllTriData) const
{
	return (BrickDataReceipt.IsValid() && Indices->Num() > 0);
}

/** Once async physics cook is done, create needed state */
void UMRMeshBodyHolder::FinishPhysicsAsyncCook(bool bSuccess, UBodySetup* FinishedBodySetup)
{
	UMRMeshComponent* MRMeshComponent = Cast<UMRMeshComponent>(GetOuter());
	if (MRMeshComponent)
	{
		FTransform BodyTransform = MRMeshComponent->GetComponentTransform();
		BodyInstance.TermBody();
		BodyInstance.InitBody(BodySetup, BodyTransform, MRMeshComponent, GetWorld()->GetPhysicsScene());
		BodyInstance.CopyRuntimeBodyInstancePropertiesFrom(MRMeshComponent->GetBodyInstance());

		MRMeshComponent->SuggestNavMeshUpdate();
	}
}

