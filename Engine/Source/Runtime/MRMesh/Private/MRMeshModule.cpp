// Copyright Epic Games, Inc. All Rights Reserved.

#include "MRMeshModule.h"
#include "Modules/ModuleManager.h"

class FMRMeshModule : public IMRMeshModule
{
};

IMPLEMENT_MODULE(FMRMeshModule, MRMesh);