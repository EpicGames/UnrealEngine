// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockDataMeshTrackerComponent.h"
#include "Engine/Engine.h"
#include "Engine/GameEngine.h"
#include "GameFramework/WorldSettings.h"
#if WITH_EDITOR
#include "Editor.h"
#endif
#include "MRMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MockDataMeshTrackerComponent)



class FMockDataMeshTrackerImpl
{
public:
	FMockDataMeshTrackerImpl()
		: MeshBrickIndex(0)
	{
		const int VertCount = 16;
		FVector Verts[VertCount] =
		{
			FVector(0.0f,0.0f,0.0f),
			FVector(10.0f,0.0f,0.0f),
			FVector(20.0f,0.0f,0.0f),
			FVector(30.0f,0.0f,0.0f),

			FVector(0.0f,10.0f,0.0f),
			FVector(10.0f,10.0f,10.0f),
			FVector(20.0f,10.0f,10.0f),
			FVector(30.0f,10.0f,0.0f),

			FVector(0.0f,20.0f,0.0f),
			FVector(10.0f,20.0f,10.0f),
			FVector(20.0f,20.0f,10.0f),
			FVector(30.0f,20.0f,0.0f),

			FVector(0.0f,30.0f,0.0f),
			FVector(10.0f,30.0f,0.0f),
			FVector(20.0f,30.0f,0.0f),
			FVector(30.0f,30.0f,0.0f),
		};
		FVector Normals[VertCount];
		const FVector Center(0.15f, 0.15f, 0.0f);
		for (int32 i = 0; i < VertCount; ++i)
		{
			Normals[i] = (Verts[i] - Center);
			Normals[i].Normalize();
		}
		const int32 IndexCount = 54;
		MRMESH_INDEX_TYPE Indices[IndexCount] =
		{
			0,4,5,
			0,5,1,
			1,5,6,
			1,6,2,
			2,6,7,
			2,7,3,

			4,8,9,
			4,9,5,
			5,9,10,
			5,10,6,
			6,10,11,
			6,11,7,

			8,12,13,
			8,13,9,
			9,13,14,
			9,14,10,
			10,14,15,
			10,15,11
		};

		const int NumBlocks = 4;
		RawMockMeshData.AddDefaulted(NumBlocks);

		for (int32 i = 0; i < NumBlocks; ++i)
		{
			FRawMockMeshData& Data = RawMockMeshData[i];
			Data.Vertices.Reserve(VertCount);
			Data.Normals.Reserve(VertCount);
			for (int32 j = 0; j < VertCount; ++j)
			{
				Data.Vertices.Add(Verts[j]);			
				Data.Normals.Add(Normals[j]);
			}
			Data.Indices.Reserve(IndexCount);
			for (int32 j = 0; j < IndexCount; ++j)
			{
				Data.Indices.Add(Indices[j]);
			}
		}

		// shift each block over by 0.3 to make a strip
		for (int32 i = 0; i < NumBlocks; ++i)
		{
			FRawMockMeshData& Data = RawMockMeshData[i];
			for (int32 j = 0; j < VertCount; ++j)
			{
				Data.Vertices[j].X += i * 30.0f;
			}
		}
	};

public:
	// Next ID for bricks created with MR Mesh
	int32 MeshBrickIndex = 0;

	struct FRawMockMeshData
	{
		TArray<FVector> Vertices;
		TArray<FVector> Normals;
		TArray<MRMESH_INDEX_TYPE> Indices;
	};
	TArray<FRawMockMeshData> RawMockMeshData;

	// Map of Raw mesh block IDs to MR Mesh brick IDs
	TMap<uint32, uint64> MeshBrickCache;

	// Keep a copy of the mesh data here.  MRMeshComponent will use it from the game and render thread.
	struct FCachedMeshData
	{
		typedef TSharedPtr<FCachedMeshData, ESPMode::ThreadSafe> SharedPtr;
		
		FMockDataMeshTrackerImpl* Owner = nullptr;
		
		IMRMesh::FBrickId BrickId = 0;
		TArray<FVector> OffsetVertices;
		TArray<FVector3f> WorldVertices;
		TArray<FVector> Normals;
		TArray<FVector2D> UV0;
		TArray<FColor> VertexColors;
		TArray<FPackedNormal> Tangents;
		TArray<MRMESH_INDEX_TYPE> Triangles;
		TArray<float> Confidence;

		// Disconnect this FCachedMeshData so if it is deleted after FMockDataMeshTrackerImpl is deleted it does not try to call back in to add itself to the free list.
		void Disconnect()
		{
			Owner = nullptr;
		}

		// Reset this FCachedMeshData so that it can be reused later.
		void Recycle(SharedPtr& MeshData)
		{
			FMockDataMeshTrackerImpl* TempOwner = Owner;
			Owner = nullptr;
	
			BrickId = 0;
			OffsetVertices.Reset();
			WorldVertices.Reset();
			Triangles.Reset();
			Normals.Reset();
			UV0.Reset();
			VertexColors.Reset();
			Tangents.Reset();
			Confidence.Reset();
	
			if (TempOwner)
			{
				TempOwner->FreeMeshDataCache(MeshData);
			}
		}

		void Init(FMockDataMeshTrackerImpl* InOwner)
		{
			check(!Owner);
			Owner = InOwner;
		}
	};


	// This receipt will be kept in the FSendBrickDataArgs to ensure the cached data outlives MRMeshComponent use of it.
	class FMeshTrackerComponentBrickDataReceipt : public IMRMesh::FBrickDataReceipt
	{
	public:
		FMeshTrackerComponentBrickDataReceipt(FCachedMeshData::SharedPtr& MeshData) :
			CachedMeshData(MeshData)
		{
		}
		~FMeshTrackerComponentBrickDataReceipt() override
		{
			CachedMeshData->Recycle(CachedMeshData);
		}
	private:
		FCachedMeshData::SharedPtr CachedMeshData;
	};
	
	FCachedMeshData::SharedPtr AquireMeshDataCache()
	{
		if (FreeCachedMeshDatas.Num() > 0)
		{
			FScopeLock ScopeLock(&FreeCachedMeshDatasMutex);
			FCachedMeshData::SharedPtr CachedMeshData(FreeCachedMeshDatas.Pop(EAllowShrinking::No));
			CachedMeshData->Init(this);
			return CachedMeshData;
		}
		else
		{
			FCachedMeshData::SharedPtr CachedMeshData(new FCachedMeshData());
			CachedMeshData->Init(this);
			CachedMeshDatas.Add(CachedMeshData);
			return CachedMeshData;
		}
	}

	void FreeMeshDataCache(FCachedMeshData::SharedPtr& DataCache)
	{
		FScopeLock ScopeLock(&FreeCachedMeshDatasMutex);
		FreeCachedMeshDatas.Add(DataCache);
	}

	FVector BoundsCenter;
	FQuat BoundsRotation;

	bool Create(const UMockDataMeshTrackerComponent& MeshTrackerComponent)
	{
		return true;
	}

	void BeginDestroy()
	{
		// Drop all the CachedMeshDatas references so that they will be destroyed unless a receipt is still hanging onto them.
		// Disconnect them so that they do not try to call back into this to be put in the FreeCachedMeshDatas array.
		for (FCachedMeshData::SharedPtr& Data : CachedMeshDatas)
		{
			Data->Disconnect();
		}
		CachedMeshDatas.Empty();
		FreeCachedMeshDatas.Empty();
	}

	void Destroy()
	{
	}

private:

	// A free list to recycle the CachedMeshData instances.  
	TArray<FCachedMeshData::SharedPtr> CachedMeshDatas;
	TArray<FCachedMeshData::SharedPtr> FreeCachedMeshDatas;
	FCriticalSection FreeCachedMeshDatasMutex; //The free list may be pushed/popped from multiple threads.
};

UMockDataMeshTrackerComponent::UMockDataMeshTrackerComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, VertexColorFromConfidenceZero(FLinearColor::Red)
	, VertexColorFromConfidenceOne(FLinearColor::Blue)
	, Impl(new FMockDataMeshTrackerImpl())
{

	// Make sure this component ticks
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;

	bAutoActivate = true;

	BlockVertexColors.Add(FColor::Blue);
	BlockVertexColors.Add(FColor::Red);
	BlockVertexColors.Add(FColor::Green);
	BlockVertexColors.Add(FColor::Yellow);
	BlockVertexColors.Add(FColor::Cyan);
	BlockVertexColors.Add(FColor::Magenta);

#if WITH_EDITOR
	if (GIsEditor)
	{
		FEditorDelegates::PrePIEEnded.AddUObject(this, &UMockDataMeshTrackerComponent::PrePIEEnded);
	}
#endif
}

UMockDataMeshTrackerComponent::~UMockDataMeshTrackerComponent()
{
	delete Impl;
}

void UMockDataMeshTrackerComponent::ConnectMRMesh(UMRMeshComponent* InMRMeshPtr)
{
	if (!InMRMeshPtr)
	{
		UE_LOGF(LogMockMeshDataTracker, Warning,
			"MRMesh given is not valid. Ignoring this connect.");
		return;
	}
	else if (MRMesh)
	{
		UE_LOGF(LogMockMeshDataTracker, Warning,
			"MeshTrackerComponent already has a MRMesh connected.  Ignoring this connect.");
		return;
	}
	else if (InMRMeshPtr->IsConnected())
	{
		UE_LOGF(LogMockMeshDataTracker, Warning,
			"MRMesh is already connected to a UMockDataMeshTrackerComponent. Ignoring this connect.");
		return;
	}
	else
	{
		InMRMeshPtr->SetConnected(true);
		MRMesh = InMRMeshPtr;
	}
}

void UMockDataMeshTrackerComponent::DisconnectMRMesh(class UMRMeshComponent* InMRMeshPtr)
{
	if (!MRMesh)
	{
		UE_LOGF(LogMockMeshDataTracker, Warning,
			"MeshTrackerComponent MRMesh is already disconnected. Ignoring this disconnect.");
		return;
	}
	else if (InMRMeshPtr != MRMesh)
	{
		UE_LOGF(LogMockMeshDataTracker, Warning,
			"MeshTrackerComponent MRMesh given is not the MRMesh connected. "
				 "Ignoring this disconnect.");
		return;
	}
	else
	{
		check(MRMesh->IsConnected());
		MRMesh->SetConnected(false);
	}
	MRMesh = nullptr;
}

#if WITH_EDITOR
void UMockDataMeshTrackerComponent::PostEditChangeProperty(FPropertyChangedEvent& e)
{
	if (e.Property != nullptr)
	{
		UE_LOGF(LogMockMeshDataTracker, Log, "PostEditChangeProperty is changing MLMeshingSettings");
	}

	Super::PostEditChangeProperty(e);
}
#endif

void UMockDataMeshTrackerComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (!MRMesh)
	{
		return;
	}

	if (!Impl->Create(*this))
	{
		return;
	}

	//// Update the bounding box within which we scan for geometry.
	//FTransform PoseInverse = UHeadMountedDisplayFunctionLibrary::GetTrackingToWorldTransform(this).Inverse();
	//PoseInverse.ConcatenateRotation(BoundingVolume->GetComponentQuat());
	//Impl->BoundsCenter = PoseInverse.TransformPosition(BoundingVolume->GetComponentLocation());
	//Impl->BoundsRotation = PoseInverse.GetRotation();

	//const float WorldToMetersScale = GWorld->GetWorldSettings()->WorldToMeters;

	// Make sure MR Mesh is at 0,0,0 (verts received from meshing are in world space)
	MRMesh->SendRelativeTransform(FTransform::Identity);

	CurrentTime += DeltaTime;

	if (ScanWorld && CurrentTime > LastUpdateTime + UpdateInterval)
	{
		LastUpdateTime = CurrentTime;
		UpdateCount += 1;

		static int MockDataPattern = 0;
		if (MockDataPattern == 0)
		{
			// Cycle adding, updating, leaving alone, and removing blocks.

			check(NumBlocks >= 3);
			// Add one block
			const int32 AddBlockIndex = (UpdateCount) % NumBlocks;
			// Update one block
			const int32 UpdateBlockIndex = (UpdateCount - 1) % NumBlocks;
			// Remove oldest block
			const int32 RemoveBlockIndex = (UpdateCount - NumBlocks + 1) % NumBlocks;

			UE_LOGF(LogMockMeshDataTracker, Log, "TickComponent is updating Add: %i Update: %i Remove: %i", AddBlockIndex, UpdateBlockIndex, RemoveBlockIndex);

			UpdateBlock(AddBlockIndex);
			UpdateBlock(UpdateBlockIndex);
			RemoveBlock(RemoveBlockIndex);
		}
		else
		{
			// Add then update 4 blocks.
			UE_LOGF(LogMockMeshDataTracker, Log, "TickComponent is adding 4 blocks");
			UpdateBlock(0);
			UpdateBlock(1);
			UpdateBlock(2);
			UpdateBlock(3);
		}

		static bool bStopScanningEveryUpdate = false;
		if (bStopScanningEveryUpdate)
		{
			ScanWorld = false;
		}
	}
}

void UMockDataMeshTrackerComponent::RemoveBlock(int32 BlockIndex)
{
	// Delete the brick and its cache entry
	if (Impl->MeshBrickCache.Contains(BlockIndex))
	{
		const static TArray<FVector3f> EmptyVertices;
		const static TArray<FVector2D> EmptyUVs;
		const static TArray<FPackedNormal> EmptyTangents;
		const static TArray<FColor> EmptyVertexColors;
		const static TArray<MRMESH_INDEX_TYPE> EmptyTriangles;
		const auto& BrickId = Impl->MeshBrickCache[BlockIndex];
		static_cast<IMRMesh*>(MRMesh)->SendBrickData(IMRMesh::FSendBrickDataArgs
			{
				nullptr,
				BrickId,
				EmptyVertices,
				EmptyUVs,
				EmptyTangents,
				EmptyVertexColors,
				EmptyTriangles
			}
		);
								
		//if (OnMeshTrackerUpdated.IsBound())
		//{
		//	const static TArray<FVector> EmptyNormals;
		//	const static TArray<float> EmptyConfidence;
		//	const static TArray<int32> EmptyTriangles2;
		//	OnMeshTrackerUpdated.Broadcast((uint32)BlockIndex, EmptyVertices, EmptyTriangles2, EmptyNormals, EmptyConfidence);
		//}

		Impl->MeshBrickCache.Remove(BlockIndex);
	}
}

void UMockDataMeshTrackerComponent::UpdateBlock(int32 BlockIndex)
{
	// Create a brick index for any new mesh block
	bool bBlockIsNew = false;
	if (!Impl->MeshBrickCache.Contains(BlockIndex))
	{
		Impl->MeshBrickCache.Add(BlockIndex, Impl->MeshBrickIndex++);
		bBlockIsNew = true;
	}

	const FMockDataMeshTrackerImpl::FRawMockMeshData& RawMeshData = Impl->RawMockMeshData[BlockIndex];

	// Acquire mesh data cache and mark its brick ID
	FMockDataMeshTrackerImpl::FCachedMeshData::SharedPtr CurrentMeshDataCache = Impl->AquireMeshDataCache();
	const auto& BrickId = Impl->MeshBrickCache[BlockIndex];
	CurrentMeshDataCache->BrickId = BrickId;

	// Pull vertices
	const FVector UpdateShiftVector = bBlockIsNew ? FVector::ZeroVector : FVector(0.0f, 0.0f, 15.0f);  // when we update we will shift the verts up a little, so we can see that something happened.
	const FVector VertexOffset = FVector::ZeroVector; // If this was the inverse of the worldToTrackingTransform position the mesh would be located in tracker space, but we don't want a dependency on HMD here.  
	const int32 VertexCount = RawMeshData.Vertices.Num();
	CurrentMeshDataCache->OffsetVertices.Reserve(VertexCount);
	CurrentMeshDataCache->WorldVertices.Reserve(VertexCount);
	for (int32 v = 0; v < VertexCount; ++ v)
	{
			CurrentMeshDataCache->OffsetVertices.Add(RawMeshData.Vertices[v] - VertexOffset + UpdateShiftVector);
			CurrentMeshDataCache->WorldVertices.Add(FVector3f(RawMeshData.Vertices[v] + UpdateShiftVector));
	}

	// Pull indices
	const int32 IndexCount = RawMeshData.Indices.Num();
	CurrentMeshDataCache->Triangles.Reserve(IndexCount);
	for (int32 i = 0; i < IndexCount; ++ i)
	{
		CurrentMeshDataCache->Triangles.Add(static_cast<MRMESH_INDEX_TYPE>(RawMeshData.Indices[i]));
	}

	// Pull normals
	CurrentMeshDataCache->Normals.Reserve(VertexCount);
	if (RequestNormals)
	{
		for (int32 n = 0; n < VertexCount; ++ n)
		{
			CurrentMeshDataCache->Normals.Add(RawMeshData.Normals[n]);
		}
	}
	// If no normals were provided we need to pack fake ones for Vulkan
	else
	{
		for (int32 n = 0; n < VertexCount; ++ n)
		{
			FVector FakeNormal = CurrentMeshDataCache->OffsetVertices[n];
			FakeNormal.Normalize();
			CurrentMeshDataCache->Normals.Add(FakeNormal);
		}
	}

	// Calculate and pack tangents
	CurrentMeshDataCache->Tangents.Reserve(VertexCount * 2);
	for (int32 t = 0; t < VertexCount; ++ t)
	{
		const FVector& Norm = CurrentMeshDataCache->Normals[t];

		// Calculate tangent
		auto Perp = Norm.X < Norm.Z ? 
			FVector(1.0f, 0.0f, 0.0f) : FVector(0.0f, 1.0f, 0.0f);
		auto Tang = FVector::CrossProduct(Norm, Perp);

		CurrentMeshDataCache->Tangents.Add(Tang);
		CurrentMeshDataCache->Tangents.Add(Norm);
	}

	// Pull confidence
	if (RequestVertexConfidence)
	{
		CurrentMeshDataCache->Confidence.Reserve(VertexCount);
		const float Confidence = (float)BlockIndex / (float)NumBlocks;
		for (int32 c = 0; c < VertexCount; ++c)
		{
			CurrentMeshDataCache->Confidence.Add(Confidence);
		}
	}

	// Apply chosen vertex color mode
	switch (VertexColorMode)
	{
		case EMeshTrackerVertexColorMode::Confidence:
		{
			if (RequestVertexConfidence)
			{
				CurrentMeshDataCache->VertexColors.Reserve(VertexCount);
				for (int32 v = 0; v < VertexCount; ++ v)
				{
					const FLinearColor VertexColor = FMath::Lerp(VertexColorFromConfidenceZero, 
						VertexColorFromConfidenceOne, CurrentMeshDataCache->Confidence[v]);
					CurrentMeshDataCache->VertexColors.Add(VertexColor.ToFColor(false));
				}
			}
			else
			{
				UE_LOGF(LogMockMeshDataTracker, Warning, "MeshTracker vertex color mode is Confidence "
					"but no confidence values available. Using white for all blocks.");
			}
			break;
		}
		case EMeshTrackerVertexColorMode::Block:
		{
			if (BlockVertexColors.Num() > 0)
			{
				const FColor& VertexColor = BlockVertexColors[BlockIndex % BlockVertexColors.Num()];

				CurrentMeshDataCache->VertexColors.Reserve(VertexCount);
				for (int32 v = 0; v < VertexCount; ++ v)
				{
					CurrentMeshDataCache->VertexColors.Add(VertexColor);
				}
			}
			else
			{
				UE_LOGF(LogMockMeshDataTracker, Warning, "MeshTracker vertex color mode is Block but "
					"no BlockVertexColors set. Using white for all blocks.");
			}
			break;
		}
		case EMeshTrackerVertexColorMode::None:
		{
			break;
		}
		default:
			check(false);
			break;
	}

	// To work in all rendering paths we always set a vertex color
	if (CurrentMeshDataCache->VertexColors.Num() == 0)
	{
		CurrentMeshDataCache->VertexColors.Reserve(VertexCount);
		for (int32 v = 0; v < VertexCount; ++ v)
		{
			CurrentMeshDataCache->VertexColors.Add(FColor::White);
		}
	}

	// Write UVs
	CurrentMeshDataCache->UV0.Reserve(VertexCount);
	for (int32 v = 0; v < VertexCount; ++ v)
	{
		const float FakeCoord = static_cast<float>(v) / static_cast<float>(VertexCount);
		CurrentMeshDataCache->UV0.Add(FVector2D(FakeCoord, FakeCoord));
	}

	// Create/update brick
	static_cast<IMRMesh*>(MRMesh)->SendBrickData(IMRMesh::FSendBrickDataArgs
		{
			TSharedPtr<IMRMesh::FBrickDataReceipt, ESPMode::ThreadSafe>
				(new FMockDataMeshTrackerImpl::FMeshTrackerComponentBrickDataReceipt(CurrentMeshDataCache)),
			CurrentMeshDataCache->BrickId,
			CurrentMeshDataCache->WorldVertices,
			CurrentMeshDataCache->UV0,
			CurrentMeshDataCache->Tangents,
			CurrentMeshDataCache->VertexColors,
			CurrentMeshDataCache->Triangles
		}
	);

	// Broadcast that a mesh was updated
	if (OnMeshTrackerUpdated.IsBound())
	{
		if (sizeof(MRMESH_INDEX_TYPE) == sizeof(uint32))
		{
			// Hack because blueprints don't support uint32.
			TArray<int32> Triangles(reinterpret_cast<const int32*>(CurrentMeshDataCache->
				Triangles.GetData()), CurrentMeshDataCache->Triangles.Num());
			OnMeshTrackerUpdated.Broadcast((int32)CurrentMeshDataCache->BrickId, CurrentMeshDataCache->OffsetVertices,
				Triangles, CurrentMeshDataCache->Normals, CurrentMeshDataCache->Confidence);
		}
	}
}

void UMockDataMeshTrackerComponent::BeginDestroy()
{
	Impl->BeginDestroy();

	if (MRMesh != nullptr)
	{
		DisconnectMRMesh(MRMesh);
	}
	Super::BeginDestroy();
}

void UMockDataMeshTrackerComponent::FinishDestroy()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		FEditorDelegates::PrePIEEnded.RemoveAll(this);
	}
#endif
	Impl->Destroy();
	Super::FinishDestroy();
}

#if WITH_EDITOR
void UMockDataMeshTrackerComponent::PrePIEEnded(bool bWasSimulatingInEditor)
{
	Impl->Destroy();
}
#endif

