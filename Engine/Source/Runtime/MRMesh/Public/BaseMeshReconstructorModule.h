// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"

class IMRMesh;

class FBaseMeshReconstructorModule : public IModuleInterface, public IModularFeature
{

public:
	//
	// MODULAR FEATURE SUPPORT
	//

	/**
	 * Part of the pattern for supporting modular features.
	 *
	 * @return the name of the feature.
	 */
	static FName GetModularFeatureName()
	{
		static const FName ModularFeatureName = FName("MeshReconstructor");
		return ModularFeatureName;
	}
	
	/**
	 * Singleton-like access to a MeshReconstructorModule.
	 *
	 * @return Returns reference to the highest priority MeshReconstructorModule module
	 */
	static inline FBaseMeshReconstructorModule& Get()
	{
		//@todo implement priority for choosing the most appropriate implementation

		TArray<FBaseMeshReconstructorModule*> MeshReconstructors = IModularFeatures::Get().GetModularFeatureImplementations<FBaseMeshReconstructorModule>(GetModularFeatureName());
		check(MeshReconstructors.Num() > 0);
		return *MeshReconstructors[0];
	}

	/**
	 * Check to see that there is a MeshReconstructor module available.
	 *
	 * @return True if there exists a MeshReconstructor module registered.
	 */
	static inline bool IsAvailable()
	{
		return IModularFeatures::Get().IsModularFeatureAvailable(GetModularFeatureName());
	}

	/**
	 * Register module as a MeshReconstructor on startup.
	 */
	virtual void StartupModule() override
	{
		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
	}

};