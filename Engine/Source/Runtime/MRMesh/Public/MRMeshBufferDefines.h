// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#ifndef WANTS_MRMESH_UVS
	#if PLATFORM_WINDOWS
		#define WANTS_MRMESH_UVS 1
	#elif PLATFORM_IOS
		#define WANTS_MRMESH_UVS 0
	#endif
#endif

#ifndef WANTS_MRMESH_TANGENTS
	#if PLATFORM_WINDOWS
		#define WANTS_MRMESH_TANGENTS 1
	#elif PLATFORM_IOS
		#define WANTS_MRMESH_TANGENTS 0
	#endif
#endif

#ifndef WANTS_MRMESH_COLORS
	#if PLATFORM_WINDOWS
		#define WANTS_MRMESH_COLORS 1
	#elif PLATFORM_IOS
		#define WANTS_MRMESH_COLORS 0
	#endif
#endif

#ifndef MRMESH_INDEX_TYPE
	#define MRMESH_INDEX_TYPE uint32
#endif
