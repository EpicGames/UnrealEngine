// Copyright Epic Games, Inc. All Rights Reserved.

// MR stands for Mixed Reality, but this component could possibly have more general use.
// It was specifically designed to support the case for Augmented Reality world mesh detection.  Where a device scans the real world and provides a mesh representing the geometry
// of the real world environment.  This mesh is expected to arrive in chunks called 'bricks' over time and for bricks to be added, removed, or updated.
// We provide the ability to render the mesh, to generate collision from it, and to generate ai navigation data on that collision.
// We do not intend to support using the MRMesh as a physics body that could be moved.
// It is somewhat similar to ProceduralMeshComponent.

#pragma once

#include "CoreMinimal.h"
#include "Components/PrimitiveComponent.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "PackedNormal.h"
#include "MRMeshBufferDefines.h"

#include "MRMeshComponent.generated.h"

class UMaterial;
class FBaseMeshReconstructorModule;
class UMeshReconstructorBase;
struct FDynamicMeshVertex;

DECLARE_STATS_GROUP(TEXT("MRMesh"), STATGROUP_MRMESH, STATCAT_Advanced);
DEFINE_LOG_CATEGORY_STATIC(LogMrMesh, Warning, All);

class IMRMesh
{
public:
	struct FBrickDataReceipt
	{
		// Optionally subclass and use receipt.  For example: to release the buffers FSendBrickDataArgs has references to.
		virtual ~FBrickDataReceipt() = default;
	};

	typedef uint64 FBrickId;
	struct FSendBrickDataArgs
	{
		// Note: many members are references!  Be aware of the lifetime of the underlying data.  FBrickDataReceipt is intended to help manage that.
		TSharedPtr<FBrickDataReceipt, ESPMode::ThreadSafe> BrickDataReceipt;
		const FBrickId BrickId = 0;
		const TArray<FVector3f>& PositionData;
		const TArray<FVector2D>& UVData;
		const TArray<FPackedNormal>& TangentXZData;
		const TArray<FColor>& ColorData;
		const TArray<MRMESH_INDEX_TYPE>& Indices;
		const FBox Bounds = FBox(EForceInit::ForceInit);
	};

	virtual void SetConnected(bool value) = 0;
	virtual bool IsConnected() const = 0;

	virtual void SendRelativeTransform(const FTransform& Transform) = 0;
	virtual void SendBrickData(FSendBrickDataArgs Args) = 0;
	virtual void Clear() = 0;
	virtual void ClearAllBrickData() = 0;
};

// Because physics cooking uses GetOuter() to get the IInterface_CollisionDataProvider and provides no way to determine which physics body it
// is currently working on we are wrapping each body in this Holder so that it can be the Outer and provide the correct data.
UCLASS(transient)
class UMRMeshBodyHolder : public UObject, public IInterface_CollisionDataProvider
{
public:
	GENERATED_BODY()

	void Initialize(IMRMesh::FBrickId InBrickId);
	void Update(const IMRMesh::FSendBrickDataArgs& Args);
	void AbortCook();
	void ReleaseArgData();
	void Cleanup();

	//~ Begin Interface_CollisionDataProvider Interface
	virtual bool GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData) override;
	virtual bool GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const override;
	virtual bool ContainsPhysicsTriMeshData(bool InUseAllTriData) const override;
	virtual bool WantsNegXTriMesh() override { return false; }
	//~ End Interface_CollisionDataProvider Interface

	/** Once async physics cook is done, create needed state */
	void FinishPhysicsAsyncCook(bool bSuccess, UBodySetup* FinishedBodySetup);

	UPROPERTY(transient)
	TObjectPtr<class UBodySetup> BodySetup;

	UPROPERTY(transient)
	FBodyInstance BodyInstance;

	// Hold references to the bulk data which is owned externally, like FSendBrickDataArgs.  The receipt keeps the references valid.
	TSharedPtr<IMRMesh::FBrickDataReceipt, ESPMode::ThreadSafe> BrickDataReceipt;
	IMRMesh::FBrickId BrickId = 0;
	const TArray<FVector3f>* PositionData = nullptr;
	const TArray<MRMESH_INDEX_TYPE>* Indices = nullptr;
	FBox Bounds = FBox(EForceInit::ForceInit);

	bool bCookInProgress = false;
};



DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMRMeshBrickDataUpdatedDelegate, const UMRMeshComponent*, const IMRMesh::FSendBrickDataArgs&);

UCLASS(hideCategories=(Physics), meta = (BlueprintSpawnableComponent), ClassGroup = Rendering, MinimalAPI)
class UMRMeshComponent : public UPrimitiveComponent, public IMRMesh
{
public:
	friend class FMRMeshProxy;

	GENERATED_UCLASS_BODY()

	MRMESH_API virtual void BeginPlay() override;
	MRMESH_API void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	UFUNCTION(BlueprintPure, Category = "Mesh Reconstruction")
	bool IsConnected() const override { return bConnected; }

	void SetConnected(bool value) override { bConnected = value; }
	virtual void SendRelativeTransform(const FTransform& Transform) override { SetRelativeTransform(Transform); }

	/**
	* Force navmesh generation to run using the current collision data.  This will run even if the collision data has not been udpated! Unless you are changing navmesh settings or similar RequestNavMeshUpdate is reccomended.
	*/	
	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API void ForceNavMeshUpdate();

	/**
	* Generate nav mesh if collision data has changed since the last nav mesh generation.  
	*/
	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API void RequestNavMeshUpdate();

	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API void Clear() override;

	// UPrimitiveComponent.. public BP function needs to stay public to avoid nativization errors. (RR)
	MRMESH_API virtual void SetMaterial(int32 ElementIndex, class UMaterialInterface* InMaterial) override;
	MRMESH_API virtual class UMaterialInterface* GetMaterial(int32 ElementIndex) const override;

	// Set the wireframe material.
	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API virtual void SetWireframeMaterial(class UMaterialInterface* InMaterial);

	/** Updates from iOS */
	MRMESH_API void UpdateMesh(const FVector& InLocation, const FQuat& InRotation, const FVector& Scale, TArray<FVector>& Vertices, TArray<MRMESH_INDEX_TYPE>& Indices, TArray<FVector2D> UVData = {}, TArray<FPackedNormal> TangentXZData = {}, TArray<FColor> ColorData = {});
	MRMESH_API void UpdateMesh(const FVector& InLocation, const FQuat& InRotation, const FVector& Scale, TArray<FVector3f>& Vertices, TArray<MRMESH_INDEX_TYPE>& Indices, TArray<FVector2D> UVData = {}, TArray<FPackedNormal> TangentXZData = {}, TArray<FColor> ColorData = {});

	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API void SetEnableMeshOcclusion(bool bEnable);
	
	UFUNCTION(BlueprintPure, Category = "Mesh Reconstruction")
	bool GetEnableMeshOcclusion() const { return bEnableOcclusion; }
	
	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API void SetUseWireframe(bool bUseWireframe);
	
	UFUNCTION(BlueprintPure, Category = "Mesh Reconstruction")
	bool GetUseWireframe() const { return bUseWireframe; }
	
	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API void SetWireframeColor(const FLinearColor& InColor);
	
	UFUNCTION(BlueprintPure, Category = "Mesh Reconstruction")
	const FLinearColor& GetWireframeColor() const { return WireframeColor; }
	
	MRMESH_API UMaterialInterface* GetMaterialToUse() const;
	
	FOnMRMeshBrickDataUpdatedDelegate& OnBrickDataUpdated() { return OnBrickDataUpdatedDelegate; }

protected:
	MRMESH_API virtual void OnActorEnableCollisionChanged() override;
	MRMESH_API virtual void UpdatePhysicsToRBChannels() override;
public:
	MRMESH_API virtual void SetCollisionObjectType(ECollisionChannel Channel) override;
	MRMESH_API virtual void SetCollisionResponseToChannel(ECollisionChannel Channel, ECollisionResponse NewResponse) override;
	MRMESH_API virtual void SetCollisionResponseToAllChannels(ECollisionResponse NewResponse) override;
	MRMESH_API virtual void SetCollisionResponseToChannels(const FCollisionResponseContainer& NewResponses) override;
	MRMESH_API virtual void SetCollisionEnabled(ECollisionEnabled::Type NewType) override;
	MRMESH_API virtual void SetCollisionProfileName(FName InCollisionProfileName, bool bUpdateOverlaps=true) override;

	MRMESH_API virtual void SetWalkableSlopeOverride(const FWalkableSlopeOverride& NewOverride) override;

	void SetNeverCreateCollisionMesh(bool bNeverCreate) { bNeverCreateCollisionMesh = bNeverCreate; }
	void SetEnableNavMesh(bool bEnable) { bUpdateNavMeshOnMeshUpdate = bEnable;  }
	MRMESH_API void SuggestNavMeshUpdate();

	/** Trackers feeding mesh data to this component may want to know when we clear our mesh data */
	DECLARE_EVENT(UMRMeshComponent, FOnClear);
	FOnClear& OnClear() { return OnClearEvent; }

private:
	//~ UPrimitiveComponent
	MRMESH_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	MRMESH_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	MRMESH_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	MRMESH_API virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;

	//~ UPrimitiveComponent
	//~ UActorComponent
	MRMESH_API virtual bool ShouldCreatePhysicsState() const override;
	MRMESH_API virtual void SendRenderDynamicData_Concurrent() override;
	//~ UActorComponent

	//~ IMRMesh
	MRMESH_API virtual void SendBrickData(FSendBrickDataArgs Args) override;
	MRMESH_API virtual void ClearAllBrickData() override;
	//~ IMRMesh

private:
	MRMESH_API void SendBrickData_Internal(IMRMesh::FSendBrickDataArgs Args);

	MRMESH_API void RemoveBodyInstance(int32 BodyIndex);
	MRMESH_API void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport) override;

	MRMESH_API void ClearAllBrickData_Internal();

	UPROPERTY(EditAnywhere, Category = Appearance)
	TObjectPtr<UMaterialInterface> Material;

	UPROPERTY(EditAnywhere, Category = Appearance)
	TObjectPtr<UMaterialInterface> WireframeMaterial;

	/** If true, MRMesh will create a renderable mesh proxy.  If false it will not, but could still provide collision. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	bool bCreateMeshProxySections = true;

	/** If true, MRMesh will automatically update its navmesh whenever any Mesh section is updated. This may be expensive. If this is disabled use ForceNavMeshUpdate to trigger a navmesh update when necessary.  Moving the component will also trigger a navmesh update.*/
	UPROPERTY(EditAnywhere, Category = MRMesh)
	bool bUpdateNavMeshOnMeshUpdate = true;

	bool bNavMeshUpdateSuggested = false;

	/** If true, MRMesh will not create a collidable ridgid body for each mesh section and can therefore never have collision.  Avoids the cost of generating collision.*/
	UPROPERTY(EditAnywhere, Category = MRMesh)
	bool bNeverCreateCollisionMesh = false;

	/**
	*	Controls whether the physics cooking should be done off the game thread. This should be used when collision geometry doesn't have to be immediately up to date (For example streaming in far away objects).  
	*   Fixing this to true for now.
	*/
	bool bUseAsyncCooking = true;

	bool bConnected = false;

	/** Whether this mesh should write z-depth to occlude meshes or not */
	bool bEnableOcclusion;
	/** Whether this mesh should draw using the wireframe material when no material is set or not */
	bool bUseWireframe;

	FOnClear OnClearEvent;
	
	FLinearColor WireframeColor = FLinearColor::White;

	FOnMRMeshBrickDataUpdatedDelegate OnBrickDataUpdatedDelegate;


	// Collision/Physics data
	UPROPERTY(Transient)
	TArray<TObjectPtr<UMRMeshBodyHolder>> BodyHolders;

	// This array is parallel to BodyHolders
	TArray<IMRMesh::FBrickId> BodyIds;
};
