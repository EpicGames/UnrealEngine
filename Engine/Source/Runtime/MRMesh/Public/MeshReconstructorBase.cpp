// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshReconstructorBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshReconstructorBase)


void UMeshReconstructorBase::StartReconstruction()
{
}

void UMeshReconstructorBase::StopReconstruction()
{
}

void UMeshReconstructorBase::PauseReconstruction()
{
}

bool UMeshReconstructorBase::IsReconstructionStarted() const
{
	return false;
}

bool UMeshReconstructorBase::IsReconstructionPaused() const
{
	return false;
}

void UMeshReconstructorBase::ConnectMRMesh(class UMRMeshComponent* Mesh)
{
}

void UMeshReconstructorBase::DisconnectMRMesh()
{
}
