// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MeshReconstructorBase.generated.h"

class IMRMesh;
class UMRMeshComponent;
struct FFrame;

USTRUCT(BlueprintType)
struct FMRMeshConfiguration
{
	GENERATED_BODY()

	bool bSendVertexColors = false;
};

UCLASS(meta=(Experimental), MinimalAPI)
class UMeshReconstructorBase : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API virtual void StartReconstruction();

	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API virtual void StopReconstruction();

	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API virtual void PauseReconstruction();

	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API virtual bool IsReconstructionStarted() const;

	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API virtual bool IsReconstructionPaused() const;

	UFUNCTION()
	MRMESH_API virtual void ConnectMRMesh(UMRMeshComponent* Mesh);

	UFUNCTION()
	MRMESH_API virtual void DisconnectMRMesh();

};
