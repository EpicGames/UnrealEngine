// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "Delegates/DelegateCombinations.h"
#include "MockDataMeshTrackerComponent.generated.h"

DEFINE_LOG_CATEGORY_STATIC(LogMockMeshDataTracker, Log, All);


/** Vertex color mode. */
UENUM(BlueprintType)
enum class EMeshTrackerVertexColorMode : uint8
{
	/** Vertex Color is not set. */
	None    UMETA(DisplayName = "No Vertex Color"),
	/*! Vertex confidence is interpolated between two specified colors. */
	Confidence  UMETA(DisplayName = "Vertex Confidence"),
	/*! Each block is given a color from a list. */
	Block  UMETA(DisplayName = "Blocks Colored")
};

/**
	The MeshTrackerComponent class manages requests for environmental mesh data, processes the results and provides
	them to the calling system. The calling system is able request environmental mesh data within a specified area.
	Various other search criteria can be set via this class's public properties.  Mesh data requests are processed
	on a separate thread.  Once a mesh data request has been processed the calling system will be notified via an
	FOnMeshTrackerUpdated broadcast.
*/
UCLASS(ClassGroup = Rendering, BlueprintType, Blueprintable, EditInlineNew, meta = (BlueprintSpawnableComponent), MinimalAPI)
class UMockDataMeshTrackerComponent
	: public USceneComponent
{
	GENERATED_UCLASS_BODY()

public:
	/** Destroys the FMeshTrackerImpl instance.*/
	MRMESH_API ~UMockDataMeshTrackerComponent();

	/**
		Sets the procedural mesh component that will store and display the environmental mesh results.
		@param InMRMeshPtr The procedural mesh component to store the query result in.
	*/
	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API void ConnectMRMesh(class UMRMeshComponent* InMRMeshPtr);

	/**
		Unlinks the current procedural mesh component from the mesh tracking system.
		@param InMRMeshPtr The procedural mesh component to unlink from the mesh tracking system.
	*/
	UFUNCTION(BlueprintCallable, Category = "Mesh Reconstruction")
	MRMESH_API void DisconnectMRMesh(class UMRMeshComponent* InMRMeshPtr);

	/**
	Delegate used by OnMeshUpdated().
	@param Index The index of the mesh section in the ProceduralMeshComponent that was updated.
	@param Vertices List of all vertices in the updated mesh section.
	@param Triangles List of all triangles in the updated mesh section.
	@param Normals List of the normals of all triangles in the updated mesh section.
	@param Confidence List of the confidence values per vertex in the updated mesh section. This can be used to determine if the user needs to scan more.
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_FiveParams(FOnMockDataMeshTrackerUpdated, int32, Index, const TArray<FVector>&, Vertices, const TArray<int32>&, Triangles, const TArray<FVector>&, Normals, const TArray<float>&, Confidence);

	/** Activated whenever new information about this mesh tracker is detected. */
	UPROPERTY(BlueprintAssignable)
	FOnMockDataMeshTrackerUpdated OnMeshTrackerUpdated;

	/** Set to true to start scanning the world for meshes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Meshing")
	bool ScanWorld = true;

	/** If true, the system will generate normals for the triangle vertices. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Meshing")
	bool RequestNormals = true;

	/**
		If true, the system will generate the mesh confidence values for the triangle vertices.
		These confidence values can be used to determine if the user needs to scan more.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Meshing")
	bool RequestVertexConfidence = false;

	/**
	* Vertex Colors can be unused, or filled with several types of information.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Meshing")
		EMeshTrackerVertexColorMode VertexColorMode = EMeshTrackerVertexColorMode::None;

	/** Colors through which we cycle when setting vertex color by block. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Meshing")
	TArray<FColor> BlockVertexColors;

	/** Color mapped to confidence value of zero. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Meshing")
	FLinearColor VertexColorFromConfidenceZero;

	/** Color mapped to confidence value of one. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Meshing")
	FLinearColor VertexColorFromConfidenceOne;



	/** Update Interval in Seconds. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Meshing|MockData")
	float UpdateInterval = 3.0f;



	/** MRMeshComponent can render and provide collision based on the Mesh data. */
	UPROPERTY(transient)
	TObjectPtr<class UMRMeshComponent> MRMesh;

	/**
		The procedural mesh generated should bake physics and update its collision data.
		TODO: Can we calculate this based on the collision flags of the UProceduralMeshComponent?
		It seems like those are not being respected directly by the component.
	*/

	/** Polls for and handles the results of the environmental mesh queries. */
	MRMESH_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;

	/** Unlinks the current procedural mesh component from the mesh tracking system. */
	MRMESH_API virtual void BeginDestroy() override;

	/** Destroys the interface object to the mesh tracking api*/
	MRMESH_API virtual void FinishDestroy() override;

#if WITH_EDITOR
	MRMESH_API virtual void PostEditChangeProperty(FPropertyChangedEvent& e) override;
#endif

private:
	class FMockDataMeshTrackerImpl *Impl;

	float LastUpdateTime = 0.0f;
	float CurrentTime = 0.0f;
	int32 UpdateCount = 0;
	int32 NumBlocks = 4;

	void UpdateBlock(int32 BlockIndex);
	void RemoveBlock(int32 BlockIndex);


#if WITH_EDITOR
private:
	void PrePIEEnded(bool bWasSimulatingInEditor);
#endif
};
